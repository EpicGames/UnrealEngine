// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeBlueprintBrush.h"
#include "CoreMinimal.h"
#include "LandscapeProxy.h"
#include "Landscape.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LandscapeBlueprintBrush)

#define LOCTEXT_NAMESPACE "Landscape"

ALandscapeBlueprintBrush::ALandscapeBlueprintBrush(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

#undef LOCTEXT_NAMESPACE
