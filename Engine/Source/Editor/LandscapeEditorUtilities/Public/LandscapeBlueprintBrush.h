// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "LandscapeBlueprintBrushBase.h"

#include "LandscapeBlueprintBrush.generated.h"

UCLASS(MinimalAPI, Abstract, Blueprintable, hidecategories = (Replication, Input, LOD, Actor, Rendering), showcategories = (Cooking))
class ALandscapeBlueprintBrush : public ALandscapeBlueprintBrushBase
{
	GENERATED_UCLASS_BODY()
};
