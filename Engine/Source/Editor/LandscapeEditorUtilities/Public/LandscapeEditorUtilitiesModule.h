// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

/**
 * LandscapeEditorUtilities module
 */
class FLandscapeEditorUtilitiesModule : public IModuleInterface
{
};
